// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LiveCodingConsole : ModuleRules
{
	public LiveCodingConsole(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Json",
				"Projects",
				"ApplicationCore",
				"Slate",
				"SlateCore",
				"StandaloneRenderer",
				"MessageLog",
				"LiveCodingServer",
				"SourceCodeAccess"
			});

		if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			PrivateDependencyModuleNames.Add("XCodeSourceCodeAccess");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "CEF3");
		}
		else if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PrivateDependencyModuleNames.Add("VisualStudioDTE");
			PrivateDependencyModuleNames.Add("VisualStudioSourceCodeAccess");
		}

		PublicIncludePathModuleNames.Add("Launch");
	}
}
