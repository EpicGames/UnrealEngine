// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using Microsoft.Extensions.Logging;

[SupportedPlatforms("Win64")]
public class LiveCodingConsoleTarget : TargetRules
{
	public LiveCodingConsoleTarget(TargetInfo Target) : base(Target)
	{
		bool UseLiveCoding2ToNotGetCompileWarning = LiveCoding.UseLiveCoding2;
		if (UseLiveCoding2ToNotGetCompileWarning && !bGenerateProjectFiles)
		{
			throw new BuildException("ERROR: LiveCodingConsole is not used when LiveCoding2 is enabled. You should not build this target");
		}

		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LaunchModuleName = "LiveCodingConsole";

		bBuildDeveloperTools = false;
		bCompileWithPluginSupport = true;
		bIncludePluginsForTargetPlatforms = true;
		bWithLiveCoding = true;

		// Editor-only data, however, is needed
		bBuildWithEditorOnlyData = true;

		// Currently this app is not linking against the engine, so we'll compile out references from Core to the rest of the engine
		bCompileAgainstApplicationCore = true;
		bCompileAgainstCoreUObject = true;
		bCompileAgainstEngine = false;

		// ICU is needed for regex during click to source code
		bCompileICU = true;
	}
}
