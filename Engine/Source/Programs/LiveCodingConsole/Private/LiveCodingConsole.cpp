// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveCodingConsole.h"
#include "RequiredProgramMainCPPInclude.h"
#include "Framework/Application/SlateApplication.h"
#include "StandaloneRenderer.h"
#include "LiveCodingConsoleStyle.h"
#include "HAL/PlatformProcess.h"
#include "SLogWidget.h"
#include "Modules/ModuleManager.h"
#include "ILiveCodingServer.h"
#include "Features/IModularFeatures.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Input/SButton.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Windows/WindowsHWrapper.h"
#include "Misc/MonitoredProcess.h"
#include "LiveCodingManifest.h"
#include "ISourceCodeAccessModule.h"
#include "Modules/ModuleInterface.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Misc/CompilationResult.h"
#include "Misc/MessageDialog.h"
#include "Misc/EngineBuildSettings.h"
#include "Misc/EngineVersion.h"

#define LOCTEXT_NAMESPACE "LiveCodingConsole"

IMPLEMENT_APPLICATION(LiveCodingConsole, "LiveCodingConsole");

#if IS_MONOLITHIC
const TCHAR* GLiveCodingEngineDir = nullptr;
const TCHAR* GLiveCodingProject = nullptr;
#endif


static void OnRequestExit()
{
	RequestEngineExit(TEXT("LiveCoding console closed"));
}

class FLiveCodingConsoleApp
{
private:
	static constexpr const TCHAR* SectionName = TEXT("LiveCodingConsole");
	static constexpr const TCHAR* DisableActionLimitKey = TEXT("bDisableActionLimit");
	static constexpr const TCHAR* ActionLimitKey = TEXT("ActionLimit");

	FCriticalSection CriticalSection;
	FSlateApplication& Slate;
	ILiveCodingServer& Server;
	TSharedPtr<SLogWidget> LogWidget;
	TSharedPtr<SWindow> Window;
	TSharedPtr<SNotificationItem> CompileNotification;
	TArray<FSimpleDelegate> MainThreadTasks;
	bool bRequestCancel;
	bool bDisableActionLimit;
	bool bHasReinstancingProcess;
	bool bWarnOnRestart;

public:
	FLiveCodingConsoleApp(FSlateApplication& InSlate, ILiveCodingServer& InServer)
		: Slate(InSlate)
		, Server(InServer)
		, bRequestCancel(false)
		, bDisableActionLimit(false)
		, bHasReinstancingProcess(false)
		, bWarnOnRestart(false)
	{
	}

	void Run()
	{
		GConfig->GetBool(SectionName, DisableActionLimitKey, bDisableActionLimit, GEngineIni);

		// open up the app window	
		LogWidget = SNew(SLogWidget);

		// Create the window
		Window = 
			SNew(SWindow)
			.Title(GetWindowTitle())
			.ClientSize(FVector2D(1200.0f, 600.0f))
			.ActivationPolicy(EWindowActivationPolicy::Never)
			.IsInitiallyMaximized(false)
			[
				SNew(SBorder)
				.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					[
						LogWidget.ToSharedRef()
					]
					
					+ SVerticalBox::Slot()
					.HAlign(HAlign_Center)
					.AutoHeight()
					.Padding(0.0f, 6.0f, 0.0f, 4.0f)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Center)
						[
							SNew(SButton)
							.Text(LOCTEXT("QuickRestart", "Quick Restart"))
							.OnClicked(FOnClicked::CreateRaw(this, &FLiveCodingConsoleApp::RestartTargets))
							.ToolTipText_Lambda([this]() { return Server.HasReinstancingProcess() ? 
								LOCTEXT("DisableQuickRestart", "Quick restarting isn't supported when re-instancing is enabled") :
								LOCTEXT("EnableQuickRestart", "Restart all live coding applications"); })
							.IsEnabled_Lambda([this]() 
								{ 
									bool bNewState = Server.HasReinstancingProcess();
									if (bNewState != bHasReinstancingProcess)
									{
										bHasReinstancingProcess = bNewState;
										if (bHasReinstancingProcess)
										{
											LogWidget->AppendLine(GetLogColor(ELiveCodingLogVerbosity::Warning), TEXT("Quick restart disabled when re-instancing is enabled."));
										}
										else
										{
											LogWidget->AppendLine(GetLogColor(ELiveCodingLogVerbosity::Success), TEXT("Quick restart enabled."));
										}
									}
									return !bHasReinstancingProcess;
								})
						]
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Center)
						.Padding(5)
						[
							SNew(SCheckBox)
							.IsChecked_Lambda([this]() { return bDisableActionLimit ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
							.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckBoxState) 
								{ 
									bDisableActionLimit = InCheckBoxState == ECheckBoxState::Checked;
									GConfig->SetBool(SectionName, DisableActionLimitKey, bDisableActionLimit, GEngineIni);
									GConfig->Flush(false, GEngineIni);
								})
							[
								SNew(STextBlock)
								.Text(LOCTEXT("DisableLimit", "Disable action limit for this session"))
							]
						]
					]
				]
			];

		// Add the window without showing it
		Slate.AddWindow(Window.ToSharedRef(), false);

		// Show the window without stealling focus
		if (!FParse::Param(FCommandLine::Get(), TEXT("Hidden")))
		{
			HWND ForegroundWindow = GetForegroundWindow();
			if (ForegroundWindow != nullptr)
			{
				::SetWindowPos((HWND)Window->GetNativeWindow()->GetOSWindowHandle(), ForegroundWindow, 0, 0, 0, 0, SWP_NOMOVE | SWP_NOSIZE | SWP_NOACTIVATE);
			}
			Window->ShowWindow();
		}

		// Get the server interface
		Server.GetBringToFrontDelegate().BindRaw(this, &FLiveCodingConsoleApp::BringToFrontAsync);
		Server.GetLogOutputDelegate().BindRaw(this, &FLiveCodingConsoleApp::AppendLogLine);
		Server.GetShowConsoleDelegate().BindRaw(this, &FLiveCodingConsoleApp::BringToFrontAsync);
		Server.GetSetVisibleDelegate().BindRaw(this, &FLiveCodingConsoleApp::SetVisibleAsync);
		Server.GetCompileDelegate().BindRaw(this, &FLiveCodingConsoleApp::CompilePatch);
		Server.GetCompileStartedDelegate().BindRaw(this, &FLiveCodingConsoleApp::OnCompileStartedAsync);
		Server.GetCompileFinishedDelegate().BindLambda([this](ELiveCodingResult Result, const wchar_t* Message){ OnCompileFinishedAsync(Result, Message); });
		Server.GetStatusChangeDelegate().BindLambda([this](const wchar_t* Status){ OnStatusChangedAsync(Status); });

		// Start the server
		FString ProcessGroupName;
		if (FParse::Value(FCommandLine::Get(), TEXT("-Group="), ProcessGroupName))
		{
			Server.Start(*ProcessGroupName);
			Window->SetRequestDestroyWindowOverride(FRequestDestroyWindowOverride::CreateLambda([this](const TSharedRef<SWindow>&){ SetVisible(false); }));
		}
		else
		{
			LogWidget->AppendLine(GetLogColor(ELiveCodingLogVerbosity::Warning), TEXT("Running in standalone mode. Server is disabled."));
		}

		// Setting focus seems to have to happen after the Window has been added
		Slate.ClearKeyboardFocus(EFocusCause::Cleared);

		// loop until the app is ready to quit
		while (!IsEngineExitRequested())
		{
			BeginExitIfRequested();

			FTSTicker::GetCoreTicker().Tick(FApp::GetDeltaTime());

			Slate.PumpMessages();
			Slate.Tick();

			FPlatformProcess::Sleep(1.0f / 30.0f);

			// Execute all the main thread tasks
			FScopeLock Lock(&CriticalSection);
			for (FSimpleDelegate& MainThreadTask : MainThreadTasks)
			{
				MainThreadTask.Execute();
			}
			MainThreadTasks.Empty();
		}

		// NOTE: In normal operation, this is never reached.  The window's JOB system will terminate the process.

		// Make sure the window is hidden, because it might take a while for the background thread to finish.
		Window->HideWindow();

		// Shutdown the server
		Server.Stop();
	}

private:
	FText GetWindowTitle()
	{
		FString ProjectName;
		if (FParse::Value(FCommandLine::Get(), TEXT("-ProjectName="), ProjectName))
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("ProjectName"), FText::FromString(ProjectName));
			Args.Add(TEXT("Branch"), FEngineBuildSettings::IsPerforceBuild() ? FText::FromString(" - " + FEngineVersion::Current().GetBranch()) : FText::GetEmpty());
			return FText::Format(LOCTEXT("WindowTitleWithProject", "{ProjectName}{Branch} - Live Coding"), Args);
		}
		return LOCTEXT("WindowTitle", "Live Coding");
	}

	void BringToFrontAsync()
	{
		FScopeLock Lock(&CriticalSection);
		MainThreadTasks.Add(FSimpleDelegate::CreateRaw(this, &FLiveCodingConsoleApp::BringToFront));
	}

	void BringToFront()
	{
		HWND WindowHandle = (HWND)Window->GetNativeWindow()->GetOSWindowHandle();
		if (IsIconic(WindowHandle))
		{
			ShowWindow(WindowHandle, SW_RESTORE);
		}
		::SetWindowPos(WindowHandle, HWND_TOPMOST, 0, 0, 0, 0, SWP_NOMOVE | SWP_NOSIZE | SWP_NOACTIVATE);
		::SetWindowPos(WindowHandle, HWND_NOTOPMOST, 0, 0, 0, 0, SWP_NOMOVE | SWP_NOSIZE | SWP_NOACTIVATE);
	}

	static FSlateColor GetLogColor(ELiveCodingLogVerbosity Verbosity)
	{
		switch (Verbosity)
		{
		case ELiveCodingLogVerbosity::Success:
			return FSlateColor(FLinearColor::Green);
		case ELiveCodingLogVerbosity::Failure:
			return FSlateColor(FLinearColor::Red);
		case ELiveCodingLogVerbosity::Warning:
			return FSlateColor(FLinearColor::Yellow);
		default:
			return FSlateColor(FLinearColor::Gray);
		}
	}

	void AppendLogLine(ELiveCodingLogVerbosity Verbosity, const TCHAR* Text)
	{
		// SLogWidget has its own synchronization
		LogWidget->AppendLine(GetLogColor(Verbosity), MoveTemp(Text));
	}

	ELiveCodingCompileResult CompilePatch(const TArray<FString>& Targets, const TArray<FString>& ValidModules, const TSet<FString>& LazyLoadModules,
		TArray<FString>& RequiredModules, FModuleToModuleFiles& ModuleToModuleFiles, ELiveCodingCompileReason CompileReason)
	{
		// Get the UBT path
		FString Executable;
		FString Entry;
		if( GConfig->GetString( TEXT("PlatformPaths"), TEXT("UnrealBuildTool"), Entry, GEngineIni ))
		{
			Executable = FPaths::ConvertRelativePathToFull(FPaths::RootDir() / Entry);
		}
		else
		{
			Executable = FPaths::EngineDir() / TEXT("Binaries/DotNET/UnrealBuildTool/UnrealBuildTool.exe");
		}
		FPaths::MakePlatformFilename(Executable);

		// Write out the list of lazy-loaded modules for UBT to check
		FString ModulesFileName = FPaths::ConvertRelativePathToFull(FPaths::EngineIntermediateDir() / TEXT("LiveCodingModules.json"));

		// Create and open a new Json file for writing, and initialize a JsonWriter to serialize the contents
		{
			TArray<uint8> TCharJsonData;
			FMemoryWriter MemWriter(TCharJsonData, true);
			{
				TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&MemWriter);
				JsonWriter->WriteObjectStart();
				JsonWriter->WriteArrayStart(TEXT("EnabledModules"));
				for (const FString& name : ValidModules)
				{
					JsonWriter->WriteValue(name);
				}
				JsonWriter->WriteArrayEnd();
				JsonWriter->WriteArrayStart(TEXT("LazyLoadModules"));
				for (const FString& name : LazyLoadModules)
				{
					JsonWriter->WriteValue(name);
				}
				JsonWriter->WriteArrayEnd();
				JsonWriter->WriteObjectEnd();
			}
			MemWriter.Close();

			TCharJsonData.AddZeroed(sizeof(TCHAR));
			auto Utf8Data = StringCast<UTF8CHAR>((const TCHAR*)TCharJsonData.GetData());
			TSharedPtr<FArchive> Ar(IFileManager::Get().CreateFileWriter(*ModulesFileName));
			Ar->Serialize(const_cast<UTF8CHAR*>(Utf8Data.Get()), Utf8Data.Length());
		}

		// Delete the output file for non-allowed modules
		FString ModulesOutputFileName = ModulesFileName + TEXT(".out");
		IFileManager::Get().Delete(*ModulesOutputFileName);

		// Delete any existing manifest
		FString ManifestFileName = FPaths::ConvertRelativePathToFull(FPaths::EngineIntermediateDir() / TEXT("LiveCoding.json"));
		IFileManager::Get().Delete(*ManifestFileName);

		// Build the argument list
		FString Arguments;
		for (const FString& Target : Targets)
		{
			Arguments += FString::Printf(TEXT("-Target=\"%s\" "), *Target.Replace(TEXT("\""), TEXT("\"\"")));
		}
		Arguments += FString::Printf(TEXT("-LiveCoding -LiveCodingModules=\"%s\" -LiveCodingManifest=\"%s\" -WaitMutex"), *ModulesFileName, *ManifestFileName);
		if (!bDisableActionLimit && CompileReason == ELiveCodingCompileReason::Initial)
		{
			int ActionLimit = 99;
			GConfig->GetInt(SectionName, ActionLimitKey, ActionLimit, GEngineIni);
			if (ActionLimit > 0)
			{
				Arguments += FString::Printf(TEXT(" -LiveCodingLimit=%d"), ActionLimit);
			}
		}
		AppendLogLine(ELiveCodingLogVerbosity::Info, *FString::Printf(TEXT("Running %s %s"), *Executable, *Arguments));

		// Spawn UBT and wait for it to complete (or the compile button to be pressed)
		FMonitoredProcess Process(*Executable, *Arguments, true);
		Process.OnOutput().BindLambda([this](const FString& Text){ AppendLogLine(ELiveCodingLogVerbosity::Info, *(FString(TEXT("  ")) + Text)); });
		Process.Launch();
		while(Process.Update())
		{
			if (HasCancelledBuild())
			{
				AppendLogLine(ELiveCodingLogVerbosity::Warning, TEXT("Build cancelled."));
				return ELiveCodingCompileResult::Canceled;
			}
			FPlatformProcess::Sleep(0.1f);
		}

		int ReturnCode = Process.GetReturnCode();
		if (ReturnCode != 0)
		{
			ELiveCodingCompileResult CompileResult = ELiveCodingCompileResult::Failure;

			// If there are missing modules, then we always retry them
			if (FPaths::FileExists(ModulesOutputFileName))
			{
				FFileHelper::LoadFileToStringArray(RequiredModules, *ModulesOutputFileName);
				if (!RequiredModules.IsEmpty())
				{
					CompileResult = ELiveCodingCompileResult::Retry;
				}
			}

			// If we reached the live coding limit, the prompt the user to retry
			if (ReturnCode == ECompilationResult::LiveCodingLimitError)
			{
				const FText Message = LOCTEXT("LimitText", "Live Coding action limit reached.  Do you wish to compile anyway?\n\n"
				"The limit can be permanently changed or disabled by setting the ActionLimit or DisableActionLimit setting for the LiveCodingConsole program.");
				const FText Title = LOCTEXT("LimitTitle", "Live Coding Action Limit Reached");
				EAppReturnType::Type ReturnType = FMessageDialog::Open(EAppMsgType::YesNo, Message, Title);
				CompileResult = ReturnType == EAppReturnType::Yes ? ELiveCodingCompileResult::Retry : ELiveCodingCompileResult::Canceled;
			}

			if (CompileResult == ELiveCodingCompileResult::Failure)
			{
				AppendLogLine(ELiveCodingLogVerbosity::Failure, TEXT("Build failed."));
			}
			return CompileResult;
		}

		// Read the output manifest
		FString ManifestFailReason;
		FLiveCodingManifest Manifest;
		if (!Manifest.Read(*ManifestFileName, ManifestFailReason))
		{
			AppendLogLine(ELiveCodingLogVerbosity::Failure, *ManifestFailReason);
			return ELiveCodingCompileResult::Failure;
		}

		// Override the linker path
		Server.SetLinkerPath(*Manifest.LinkerPath, Manifest.LinkerEnvironment);

		// Add all the sources regardless of modification time.  The patch will exclude old ones
		for(TPair<FString, TArray<FString>>& Pair : Manifest.BinaryToObjectFiles)
		{
			for (const FString& ObjectFileName : Pair.Value)
			{
				ModuleToModuleFiles.FindOrAdd(Pair.Key).Objects.Add(ObjectFileName);
			}
		}

		// Add all of the additional libraries
		for (TPair<FString, TArray<FString>>& Pair : Manifest.Libraries)
		{
			for (const FString& Library : Pair.Value)
			{
				ModuleToModuleFiles.FindOrAdd(Pair.Key).Libraries.Add(Library);
			}
		}

		return ELiveCodingCompileResult::Success;
	}

	void CancelBuild()
	{
		FScopeLock Lock(&CriticalSection);
		bRequestCancel = true;
	}

	bool HasCancelledBuild()
	{
		FScopeLock Lock(&CriticalSection);
		return bRequestCancel;
	}

	FReply RestartTargets()
	{
		if (bWarnOnRestart)
		{
			const FText Message = LOCTEXT("RestartWarningText", "Restarting after patching while re-instancing was enabled can lead to unexpected results.\r\n\r\nDo you wish to continue?");
			const FText Title = LOCTEXT("RestartWarningTitle", "Restarting after patching while re-instancing was enabled?");
			EAppReturnType::Type ReturnType = FMessageDialog::Open(EAppMsgType::YesNo, Message, Title);
			if (ReturnType != EAppReturnType::Yes)
			{
				return FReply::Handled();
			}
		}
		Server.RestartTargets();
		return FReply::Handled();
	}

	void SetVisibleAsync(bool bVisible)
	{
		FScopeLock Lock(&CriticalSection);
		MainThreadTasks.Add(FSimpleDelegate::CreateLambda([this, bVisible](){ SetVisible(bVisible); }));
	}

	void SetVisible(bool bVisible)
	{
		if (bVisible)
		{
			if (!Window->IsVisible())
			{
				Window->ShowWindow();
			}
		}
		else
		{
			if (Window->IsVisible())
			{
				Window->HideWindow();
			}
		}
	}

	void ShowConsole()
	{
		SetVisible(true);
		BringToFront();
	}

	void OnCompileStartedAsync()
	{
		FScopeLock Lock(&CriticalSection);
		MainThreadTasks.Add(FSimpleDelegate::CreateRaw(this, &FLiveCodingConsoleApp::OnCompileStarted));
		bRequestCancel = false;
	}

	void OnCompileStarted()
	{
		if (!CompileNotification.IsValid())
		{
			ShowConsole();

			FNotificationInfo Info(FText::FromString(TEXT("Starting...")));
			Info.bFireAndForget = false;
			Info.FadeOutDuration = 0.0f;
			Info.ExpireDuration = 0.0f;
			Info.Hyperlink = FSimpleDelegate::CreateRaw(this, &FLiveCodingConsoleApp::ShowConsole);
			Info.HyperlinkText = LOCTEXT("BuildStatusShowConsole", "Show Console");
			Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("BuildStatusCancel", "Cancel"), FText(), FSimpleDelegate::CreateRaw(this, &FLiveCodingConsoleApp::CancelBuild), SNotificationItem::CS_Pending));

			CompileNotification = FSlateNotificationManager::Get().AddNotification(Info);
			CompileNotification->SetCompletionState(SNotificationItem::CS_Pending);
		}
	}

	void OnCompileFinishedAsync(ELiveCodingResult Result, const FString& Status)
	{
		FScopeLock Lock(&CriticalSection);
		MainThreadTasks.Add(FSimpleDelegate::CreateLambda([this, Result, Status](){ OnCompileFinished(Result, Status); }));
	}

	void OnCompileFinished(ELiveCodingResult Result, const FString& Status)
	{
		if(CompileNotification.IsValid())
		{
			if (Result == ELiveCodingResult::Success)
			{
				if (Server.ShowCompileFinishNotification())
				{
					CompileNotification->SetText(FText::FromString(Status));
					CompileNotification->SetCompletionState(SNotificationItem::CS_Success);
					CompileNotification->SetExpireDuration(1.5f);
					CompileNotification->SetFadeOutDuration(0.4f);
				}
				else
				{
					CompileNotification->SetExpireDuration(0.0f);
					CompileNotification->SetFadeOutDuration(0.1f);
				}
			}
			else if (HasCancelledBuild())
			{
				CompileNotification->SetExpireDuration(0.0f);
				CompileNotification->SetFadeOutDuration(0.1f);
			}
			else
			{
				if (Server.ShowCompileFinishNotification())
				{
					CompileNotification->SetText(FText::FromString(Status));
					CompileNotification->SetCompletionState(SNotificationItem::CS_Fail);
					CompileNotification->SetExpireDuration(5.0f);
					CompileNotification->SetFadeOutDuration(2.0f);
				}
				else
				{
					CompileNotification->SetExpireDuration(0.0f);
					CompileNotification->SetFadeOutDuration(0.1f);
					ShowConsole();
				}
			}

		}
		CompileNotification->ExpireAndFadeout();
		CompileNotification.Reset();

		if (Result == ELiveCodingResult::Success && bHasReinstancingProcess)
		{
			bWarnOnRestart = true;
		}
	}

	void OnStatusChangedAsync(const FString& Status)
	{
		FScopeLock Lock(&CriticalSection);
		MainThreadTasks.Add(FSimpleDelegate::CreateLambda([this, Status](){ OnCompileStatusChanged(Status); }));
	}

	void OnCompileStatusChanged(const FString& Status)
	{
		if (CompileNotification.IsValid())
		{
			CompileNotification->SetText(FText::FromString(Status));
		}
	}
};

bool LiveCodingConsoleMain(const TCHAR* CmdLine)
{
	// start up the main loop
	GEngineLoop.PreInit(CmdLine);
	check(GConfig && GConfig->IsReadyForUse());

	// Initialize high DPI mode
	FSlateApplication::InitHighDPI(true);

	{
		// Create the platform slate application (what FSlateApplication::Get() returns)
		TSharedRef<FSlateApplication> Slate = FSlateApplication::Create(MakeShareable(FPlatformApplicationMisc::CreateApplication()));

		{
			// Initialize renderer
			TSharedRef<FSlateRenderer> SlateRenderer = GetStandardStandaloneRenderer();

			// Try to initialize the renderer. It's possible that we launched when the driver crashed so try a few times before giving up.
			bool bRendererInitialized = Slate->InitializeRenderer(SlateRenderer, true);
			if (!bRendererInitialized)
			{
				// Close down the Slate application
				FSlateApplication::Shutdown();
				return false;
			}

			// Set the normal UE IsEngineExitRequested() when outer frame is closed
			Slate->SetExitRequestedHandler(FSimpleDelegate::CreateStatic(&OnRequestExit));

			// Prepare the custom Slate styles
			FLiveCodingConsoleStyle::Initialize();

			// Set the icon
			FAppStyle::SetAppStyleSet(FLiveCodingConsoleStyle::Get());

			// Load the source code access module
			ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>(FName("SourceCodeAccess"));

			// Manually load in the source code access plugins, as standalone programs don't currently support plugins.
#if PLATFORM_MAC
			IModuleInterface& XCodeSourceCodeAccessModule = FModuleManager::LoadModuleChecked<IModuleInterface>(FName("XCodeSourceCodeAccess"));
			SourceCodeAccessModule.SetAccessor(FName("XCodeSourceCodeAccess"));
#elif PLATFORM_WINDOWS
			IModuleInterface& VisualStudioSourceCodeAccessModule = FModuleManager::LoadModuleChecked<IModuleInterface>(FName("VisualStudioSourceCodeAccess"));
			SourceCodeAccessModule.SetAccessor(FName("VisualStudioSourceCodeAccess"));
#endif

			// Load the server module
			FModuleManager::Get().LoadModuleChecked<ILiveCodingServerModule>(TEXT("LiveCodingServer"));
			ILiveCodingServer& Server = IModularFeatures::Get().GetModularFeature<ILiveCodingServer>(LIVE_CODING_SERVER_FEATURE_NAME);

			// Run the inner application loop
			FLiveCodingConsoleApp App(Slate.Get(), Server);
			App.Run();

			// Unload the server module
			FModuleManager::Get().UnloadModule(TEXT("LiveCodingServer"));

			// Clean up the custom styles
			FLiveCodingConsoleStyle::Shutdown();
		}

		// Close down the Slate application
		FSlateApplication::Shutdown();
	}

	FEngineLoop::AppPreExit();
	FModuleManager::Get().UnloadModulesAtShutdown();
	FEngineLoop::AppExit();
	return true;
}

int WINAPI WinMain(HINSTANCE hCurrInstance, HINSTANCE hPrevInstance, LPSTR lpCmdLine, int nShowCmd)
{
	hInstance = hCurrInstance;
	return LiveCodingConsoleMain(GetCommandLineW())? 0 : 1;
}

#undef LOCTEXT_NAMESPACE
