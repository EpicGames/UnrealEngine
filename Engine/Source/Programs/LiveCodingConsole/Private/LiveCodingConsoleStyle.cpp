// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveCodingConsoleStyle.h"
#include "Misc/AssertionMacros.h"
#include "Containers/UnrealString.h"
#include "Misc/Paths.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"

#define RootToContentDir ContentFromEngine

TSharedPtr< FSlateStyleSet > FLiveCodingConsoleStyle::StyleSet = nullptr;

void FLiveCodingConsoleStyle::Initialize()
{
	if(!StyleSet.IsValid())
	{
		StyleSet = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleSet);
	}
}

void FLiveCodingConsoleStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet);
	ensure(StyleSet.IsUnique());
	StyleSet.Reset();
}

namespace
{
	FString ContentFromEngine(const FString& RelativePath, const TCHAR* Extension)
	{
		static const FString ContentDir = FPaths::EngineDir() / TEXT("Content/Slate");
		return ContentDir / RelativePath + Extension;
	}
}

TSharedRef< FSlateStyleSet > FLiveCodingConsoleStyle::Create()
{
	TSharedRef<FSlateStyleSet> StyleRef = MakeShareable(new FSlateStyleSet("LiveCodingServerStyle"));
	FSlateStyleSet& Style = StyleRef.Get();

	Style.SetParentStyleName("CoreStyle");


	Style.Set( "AppIcon", new IMAGE_BRUSH( "Icons/DefaultAppIcon", FVector2D(20,20)) );

	const FTextBlockStyle DefaultText = FTextBlockStyle()
		.SetFont(DEFAULT_FONT("Bold", 10))
		.SetColorAndOpacity(FSlateColor::UseForeground())
		.SetShadowOffset(FVector2D::ZeroVector)
		.SetShadowColorAndOpacity(FLinearColor::Black);

	// Set the client app styles
	Style.Set(TEXT("Code"), FTextBlockStyle(DefaultText)
		.SetFont(DEFAULT_FONT("Regular", 8))
		.SetColorAndOpacity(FSlateColor(FLinearColor::White * 0.8f))
	);

	Style.Set(TEXT("Title"), FTextBlockStyle(DefaultText)
		.SetFont(DEFAULT_FONT("Bold", 12))
	);

	Style.Set(TEXT("Status"), FTextBlockStyle(DefaultText)
		.SetColorAndOpacity(FSlateColor::UseSubduedForeground())
	);


	const FVector2D Icon16x16( 16.0f, 16.0f );
	FSlateBrush* GenericWhiteBox = new IMAGE_BRUSH( "Old/White", Icon16x16 );

	// Scrollbar
	const FScrollBarStyle ScrollBar = FScrollBarStyle()
		.SetVerticalTopSlotImage(IMAGE_BRUSH("Common/Scrollbar_Background_Vertical", FVector2D(8, 8)))
		.SetVerticalBottomSlotImage(IMAGE_BRUSH("Common/Scrollbar_Background_Vertical", FVector2D(8, 8)))
		.SetHorizontalTopSlotImage(IMAGE_BRUSH("Common/Scrollbar_Background_Horizontal", FVector2D(8, 8)))
		.SetHorizontalBottomSlotImage(IMAGE_BRUSH("Common/Scrollbar_Background_Horizontal", FVector2D(8, 8)))
		.SetNormalThumbImage(BOX_BRUSH("Common/Scrollbar_Thumb", FMargin(4.f / 16.f)))
		.SetDraggedThumbImage(BOX_BRUSH("Common/Scrollbar_Thumb", FMargin(4.f / 16.f)))
		.SetHoveredThumbImage(BOX_BRUSH("Common/Scrollbar_Thumb", FMargin(4.f / 16.f)));

	Style.Set("Log.TextBox", FEditableTextBoxStyle(FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"))
		.SetBackgroundImageNormal( BOX_BRUSH("Common/WhiteGroupBorder", FMargin(4.0f/16.0f)))
		.SetBackgroundImageHovered( BOX_BRUSH("Common/WhiteGroupBorder", FMargin(4.0f/16.0f)))
		.SetBackgroundImageFocused(BOX_BRUSH("Common/WhiteGroupBorder", FMargin(4.0f / 16.0f)))
		.SetBackgroundImageReadOnly(BOX_BRUSH("Common/WhiteGroupBorder", FMargin(4.0f / 16.0f)))
		.SetBackgroundColor( FLinearColor(0.015f, 0.015f, 0.015f) )
		.SetScrollBarStyle(ScrollBar)
		);

	return StyleRef;
}

const ISlateStyle& FLiveCodingConsoleStyle::Get()
{
	return *StyleSet;
}

#undef RootToContentDir
