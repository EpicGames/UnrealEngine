// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateStyle.h"

class FLiveCodingConsoleStyle
{
public:
	/**
	 * Set up specific styles for the crash report client app
	 */
	static void Initialize();

	/**
	 * Tidy up on shut-down
	 */
	static void Shutdown();

	/*
	 * Access to singleton style object
	 */ 
	static const ISlateStyle& Get();

private:
	static TSharedRef<FSlateStyleSet> Create();

	/** Singleton style object */
	static TSharedPtr<FSlateStyleSet> StyleSet;
};
