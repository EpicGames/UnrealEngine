// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveCodingManifest.h"
#include "Misc/FileHelper.h"

bool FLiveCodingManifest::Read(const TCHAR* FileName, FString& OutFailReason)
{
	// Read the file to a string
	FString FileContents;
	if (!FFileHelper::LoadFileToString(FileContents, FileName))
	{
		OutFailReason = FString::Printf(TEXT("Unable to read from %s"), FileName);
		return false;
	}

	// Deserialize a JSON object from the string
	TSharedPtr< FJsonObject > Object;
	TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create(FileContents);
	if ( !FJsonSerializer::Deserialize(Reader, Object) || !Object.IsValid() )
	{
		OutFailReason = FString::Printf(TEXT("Unable to parse %s"), FileName);
		return false;
	}

	// Parse the data
	if (!Parse(*Object, OutFailReason))
	{
		OutFailReason = FString::Printf(TEXT("Unable to parse %s (%s)"), FileName, *OutFailReason);
		return false;
	}

	return true;
}

bool FLiveCodingManifest::Parse(FJsonObject& Object, FString& OutFailReason)
{
	if (!Object.TryGetStringField(TEXT("LinkerPath"), LinkerPath))
	{
		OutFailReason = TEXT("missing LinkerPath field");
		return false;
	}

	const TSharedPtr<FJsonObject>* EnvironmentObject;
	if (Object.TryGetObjectField(TEXT("LinkerEnvironment"), EnvironmentObject))
	{
		for(const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Pair : EnvironmentObject->Get()->Values)
		{
			const FJsonValue* Value = Pair.Value.Get();
			if (Value->Type == EJson::String)
			{
				LinkerEnvironment.Add(FString(Pair.Key), Value->AsString());
			}
		}
	}

	const TArray<TSharedPtr<FJsonValue>>* ModulesArray;
	if (!Object.TryGetArrayField(TEXT("Modules"), ModulesArray))
	{
		OutFailReason = TEXT("missing Modules field");
		return false;
	}

	for (const TSharedPtr<FJsonValue>& ModuleValue : *ModulesArray)
	{
		if (ModuleValue->Type != EJson::Object)
		{
			OutFailReason = TEXT("invalid module object");
			return false;
		}

		const FJsonObject& ModuleObject = *ModuleValue->AsObject();

		FString OutputFile;
		if (!ModuleObject.TryGetStringField(TEXT("Output"), OutputFile))
		{
			OutFailReason = TEXT("missing module 'Output' field");
			return false;
		}

		const TArray<TSharedPtr<FJsonValue>>* Inputs;
		if (!ModuleObject.TryGetArrayField(TEXT("Inputs"), Inputs))
		{
			OutFailReason = TEXT("missing module 'Inputs' field");
			return false;
		}

		TArray<FString>& InputFiles = BinaryToObjectFiles.Add(OutputFile);
		for (const TSharedPtr<FJsonValue>& Input : *Inputs)
		{
			if (Input->Type != EJson::String)
			{
				OutFailReason = TEXT("invalid module input field");
				return false;
			}
			InputFiles.Add(Input->AsString());
		}

		const TArray<TSharedPtr<FJsonValue>>* JsonLibraries;
		if (ModuleObject.TryGetArrayField(TEXT("Libraries"), JsonLibraries))
		{
			TArray<FString>& LibraryFiles = Libraries.Add(OutputFile);
			for (const TSharedPtr<FJsonValue>& Library : *JsonLibraries)
			{
				if (Library->Type != EJson::String)
				{
					OutFailReason = TEXT("invalid module library field");
					return false;
				}
				LibraryFiles.Add(Library->AsString());
			}
		}
	}

	const TArray<TSharedPtr<FJsonValue>>* VfsArray;
	if (!Object.TryGetArrayField(TEXT("Vfs"), VfsArray))
	{
		OutFailReason = TEXT("missing Vfs field");
		return false;
	}

	for (const TSharedPtr<FJsonValue>& VfsValue : *VfsArray)
	{
		if (VfsValue->Type != EJson::Object)
		{
			OutFailReason = TEXT("invalid vfs object");
			return false;
		}

		const FJsonObject& VfsObject = *VfsValue->AsObject();

		FString VirtualPath;
		if (!VfsObject.TryGetStringField(TEXT("Virtual"), VirtualPath))
		{
			OutFailReason = TEXT("missing Vfs 'Virtual' field");
			return false;
		}

		FString LocalPath;
		if (!VfsObject.TryGetStringField(TEXT("Local"), LocalPath))
		{
			OutFailReason = TEXT("missing Vfs 'Local' field");
			return false;
		}

		VfsEntries.Emplace(VirtualPath, LocalPath);
	}

	return true;
}
