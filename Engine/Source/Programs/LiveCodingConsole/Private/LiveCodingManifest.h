// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

struct FLiveCodingManifest
{
	FString LinkerPath;
	TMap<FString, FString> LinkerEnvironment;
	TMap<FString, TArray<FString>> BinaryToObjectFiles;
	TMap<FString, TArray<FString>> Libraries;
	TArray<TPair<FString, FString>> VfsEntries;

	bool Read(const TCHAR* FileName, FString& OutFailReason);
	bool Parse(FJsonObject& Object, FString& OutFailReason);
};
