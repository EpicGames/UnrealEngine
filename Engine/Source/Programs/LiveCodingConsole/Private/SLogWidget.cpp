// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLogWidget.h"
#include "Framework/Text/SlateTextRun.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LiveCodingConsoleStyle.h"
#include "SlateOptMacros.h"
#include "HAL/FileManager.h"
#include "ISourceCodeAccessModule.h"
#include "ISourceCodeAccessor.h"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

#define LOCTEXT_NAMESPACE "LiveCoding"

//// FLogWidgetTextLayoutMarshaller ////

FLogWidgetTextLayoutMarshaller::FLogWidgetTextLayoutMarshaller()
	: TextLayout(nullptr)
{
	DefaultStyle = FTextBlockStyle()
		.SetFont( FCoreStyle::GetDefaultFontStyle( "Mono", 9 ) )
		.SetColorAndOpacity( FLinearColor::White )
		.SetSelectedBackgroundColor( FLinearColor(0.9f, 0.9f, 0.9f) );
}

FLogWidgetTextLayoutMarshaller::~FLogWidgetTextLayoutMarshaller()
{
}

void FLogWidgetTextLayoutMarshaller::SetText(const FString& SourceString, FTextLayout& TargetTextLayout)
{
	TextLayout = &TargetTextLayout;

	for(const TSharedRef<FString>& Line : Lines)
	{
		TextLayout->AddLine(FSlateTextLayout::FNewLineData(Line, TArray<TSharedRef<IRun>>()));
	}
}

void FLogWidgetTextLayoutMarshaller::GetText(FString& TargetString, const FTextLayout& SourceTextLayout)
{
	SourceTextLayout.GetAsText(TargetString);
}

void FLogWidgetTextLayoutMarshaller::Clear()
{
	Lines.Empty();
	MakeDirty();
}

void FLogWidgetTextLayoutMarshaller::AppendLine(const FSlateColor& Color, const FString& Line)
{
	TSharedRef<FString> NewLine = MakeShared<FString>(Line);
	Lines.Add(NewLine);

	if(TextLayout)
	{
		// Remove the "default" line that's added for an empty text box.
		if(Lines.Num() == 1)
		{
			TextLayout->ClearLines();
		}

		FTextBlockStyle Style = DefaultStyle;
		Style.ColorAndOpacity = Color;

		TArray<TSharedRef<IRun>> Runs;
		Runs.Add(FSlateTextRun::Create(FRunInfo(), NewLine, Style));
		TextLayout->AddLine(FSlateTextLayout::FNewLineData(NewLine, Runs));
	}
}

int32 FLogWidgetTextLayoutMarshaller::GetNumLines() const
{
	return Lines.Num();
}

//// SLogWidget ////

SLogWidget::SLogWidget()
	: bIsUserScrolledX(false)
	, bIsUserScrolledY(false)
{
}

SLogWidget::~SLogWidget()
{
}

void SLogWidget::Construct(const FArguments& InArgs)
{
	MessagesTextMarshaller = MakeShared<FLogWidgetTextLayoutMarshaller>();

	ChildSlot
		[
			SNew(SBorder)
			[
				SAssignNew(MessagesTextBox, SMultiLineEditableTextBox)
				.Style(FLiveCodingConsoleStyle::Get(), "Log.TextBox")
		        .Marshaller(MessagesTextMarshaller)
		        .IsReadOnly(true)
		        .AlwaysShowScrollbars(true)
				.SelectWordOnMouseDoubleClick(false)
		        .OnHScrollBarUserScrolled(this, &SLogWidget::OnScrollX)
		        .OnVScrollBarUserScrolled(this, &SLogWidget::OnScrollY)
				.ContextMenuExtender(this, &SLogWidget::ExtendTextBoxMenu)
			]
		];

	RegisterActiveTimer(0.03f, FWidgetActiveTimerDelegate::CreateSP(this, &SLogWidget::OnTimerElapsed));
}

void SLogWidget::ExtendTextBoxMenu(FMenuBuilder& Builder)
{
	FUIAction ClearOutputLogAction(
		FExecuteAction::CreateRaw(this, &SLogWidget::OnClearLog),
		FCanExecuteAction::CreateSP(this, &SLogWidget::CanClearLog)
	);

	Builder.AddMenuEntry(
		NSLOCTEXT("OutputLog", "ClearLogLabel", "Clear Log"),
		NSLOCTEXT("OutputLog", "ClearLogTooltip", "Clears all log messages"),
		FSlateIcon(),
		ClearOutputLogAction
	);
}

void SLogWidget::OnClearLog()
{
	// Make sure the cursor is back at the start of the log before we clear it
	MessagesTextBox->GoTo(FTextLocation(0));

	Clear();
	MessagesTextBox->Refresh();
	bIsUserScrolledX = false;
	bIsUserScrolledY = false;
}

bool SLogWidget::CanClearLog() const
{
	return MessagesTextMarshaller->GetNumLines() > 0;
}

void SLogWidget::Clear()
{
	MessagesTextMarshaller->Clear();
}

void SLogWidget::ScrollToEnd()
{
	MessagesTextBox->ScrollTo(FTextLocation(MessagesTextMarshaller->GetNumLines() - 1));
	bIsUserScrolledX = false;
	bIsUserScrolledY = false;
}

void SLogWidget::AppendLine(const FSlateColor& Color, const FString& Text)
{
	// Split any multi line text block into multiple lines while removing the \r\n or \n from the string.
	// The ParseIntoArray method will either leave the trailing blank line or eliminate all contained blank 
	// lines so we use a hand written algorithm.
	for (const TCHAR* Cur = *Text, *End = Cur + Text.Len(); Cur != End;)
	{
		const TCHAR* LineStart = Cur;
		const TCHAR* LineEnd = End;
		for (; Cur != End; ++Cur)
		{
			if (*Cur == '\n')
			{
				LineEnd = Cur;
				++Cur;
				break;
			}
			else if (*Cur == '\r')
			{
				LineEnd = Cur;
				++Cur;
				if (Cur != End && *Cur == '\n')
				{
					++Cur;
				}
				break;
			}
		}
		FScopeLock Lock(&CriticalSection);
		QueuedLines.Add(FLine{ Color, FString(LineEnd - LineStart, LineStart) });
	}
}

void SLogWidget::OnScrollX(float ScrollOffset)
{
	bIsUserScrolledX = ScrollOffset > 0.0 && !FMath::IsNearlyEqual(ScrollOffset, 0.0f);
}

void SLogWidget::OnScrollY(float ScrollOffset)
{
	bIsUserScrolledY = ScrollOffset < 1.0 && !FMath::IsNearlyEqual(ScrollOffset, 1.0f);
}

EActiveTimerReturnType SLogWidget::OnTimerElapsed(double CurrentTime, float DeltaTime)
{
	FScopeLock Lock(&CriticalSection);
	for(const FLine& QueuedLine : QueuedLines)
	{
		MessagesTextMarshaller->AppendLine(QueuedLine.Color, QueuedLine.Text);
	}
	if(!bIsUserScrolledX && !bIsUserScrolledY)
	{
		ScrollToEnd();
	}
	QueuedLines.Empty();
	return EActiveTimerReturnType::Continue;
}

bool ExtractFilepathAndLineNumber(FString& PotentialFilePath, int32& LineNumber)
{
	// Extract filename and line number using regex	
#if PLATFORM_WINDOWS
	const FRegexPattern SourceCodeRegexPattern(TEXT("([a-zA-Z]:/[^:\\n\\r()]+(h|cpp))\\s?\\(([0-9]+)\\)"));
	const int32 LineNumberCaptureGroupID = 3;
#else
	const FRegexPattern SourceCodeRegexPattern(TEXT("((//([^:/\\n]+[/])*)([^/]+)(h|cpp))\\s?\\(([0-9]+)\\)"));
	const int32 LineNumberCaptureGroupID = 6;
#endif

	FRegexMatcher SourceCodeRegexMatcher(SourceCodeRegexPattern, PotentialFilePath);
	if (SourceCodeRegexMatcher.FindNext())
	{
		PotentialFilePath = SourceCodeRegexMatcher.GetCaptureGroup(1);
		LineNumber = FCString::Strtoi(*SourceCodeRegexMatcher.GetCaptureGroup(LineNumberCaptureGroupID), nullptr, 10);
		return true;
	}

	return false;
}

FReply SLogWidget::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		// grab cursor location's line of text
		FString PotentialCodeFilePath;
		MessagesTextBox->GetCurrentTextLine(PotentialCodeFilePath);

		// Extract potential .cpp./h files file path & line number
		int32 LineNumber = 0;
		PotentialCodeFilePath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*PotentialCodeFilePath);
		if (ExtractFilepathAndLineNumber(PotentialCodeFilePath, LineNumber) && PotentialCodeFilePath.Len() && IFileManager::Get().FileSize(*PotentialCodeFilePath) != INDEX_NONE)
		{
			ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
			SourceCodeAccessModule.GetAccessor().OpenFileAtLine(PotentialCodeFilePath, LineNumber);
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE

END_SLATE_FUNCTION_BUILD_OPTIMIZATION
