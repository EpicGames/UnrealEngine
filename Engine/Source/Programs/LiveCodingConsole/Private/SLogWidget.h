// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SlateCore.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/Text/BaseTextLayoutMarshaller.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

class FLogWidgetTextLayoutMarshaller : public FBaseTextLayoutMarshaller
{
public:
	FLogWidgetTextLayoutMarshaller();
	virtual ~FLogWidgetTextLayoutMarshaller();

	// ITextLayoutMarshaller
	virtual void SetText(const FString& SourceString, FTextLayout& TargetTextLayout) override;
	virtual void GetText(FString& TargetString, const FTextLayout& SourceTextLayout) override;

	void Clear();
	void AppendLine(const FSlateColor& Color, const FString& Line);
	int32 GetNumLines() const;

private:
	FTextBlockStyle DefaultStyle;
	TArray<TSharedRef<FString>> Lines;
	FTextLayout* TextLayout;
};

class SLogWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLogWidget) { }
	SLATE_END_ARGS()

		SLogWidget();
	~SLogWidget();

	void Construct( const FArguments& InArgs );

	void Clear();
	void ScrollToEnd();
	void AppendLine(const FSlateColor& Color, const FString& Text);
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
protected:
	struct FLine
	{
		FSlateColor Color;
		FString Text;
	};

	FEditableTextBoxStyle Style;
	FCriticalSection CriticalSection;
	TArray<FLine> QueuedLines;
	TSharedPtr<FLogWidgetTextLayoutMarshaller> MessagesTextMarshaller;
	TSharedPtr<SMultiLineEditableTextBox> MessagesTextBox;
	bool bIsUserScrolledX;
	bool bIsUserScrolledY;

	void OnScrollX(float ScrollOffset);
	void OnScrollY(float ScrollOffset);
	EActiveTimerReturnType OnTimerElapsed(double CurrentTime, float DeltaTime);

	/**
	 * Extends the context menu used by the text box
	 */
	void ExtendTextBoxMenu(FMenuBuilder& Builder);

	/**
	* Called when delete all is selected
	*/
	void OnClearLog();

	/**
	 * Called to determine whether delete all is currently a valid command
	 */
	bool CanClearLog() const;
};
